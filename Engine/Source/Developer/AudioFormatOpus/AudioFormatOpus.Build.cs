// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioFormatOpus : ModuleRules
{
	protected virtual bool bWithLibOpus { get => (
			(Target.Platform == UnrealTargetPlatform.Win64) ||
			Target.IsInPlatformGroup(UnrealPlatformGroup.Linux) ||
			(Target.Platform == UnrealTargetPlatform.Mac)
		); 
	}

	public AudioFormatOpus(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("TargetPlatform");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Engine",
				"VorbisAudioDecoder",	// for VorbisChannelInfo
				"OpusAudioDecoder"
			}
		);

		if (bWithLibOpus)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"libOpus"
			);
		}

		PublicDefinitions.Add("WITH_OGGVORBIS=1");
	}
}
