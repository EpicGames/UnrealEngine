// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioFormatOpus.h"
#include "Audio.h"
#include "Serialization/MemoryWriter.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IAudioFormat.h"
#include "Interfaces/IAudioFormatModule.h"

#include "Decoders/OpusAudioInfo.h"
#include "Decoders/VorbisAudioInfo.h"	// for VorbisChannelInfo

THIRD_PARTY_INCLUDES_START
#include "opus_multistream.h"
THIRD_PARTY_INCLUDES_END

/** Use UE memory allocation or Opus */
#define USE_UE_MEM_ALLOC 1
#define SAMPLE_SIZE			( ( uint32 )sizeof( short ) )

static FName NAME_OPUS(TEXT("OPUS"));

/**
 * IAudioFormat, audio compression abstraction
**/
class FAudioFormatOpus : public IAudioFormat
{
	enum
	{
		/** Version for OPUS format, this becomes part of the DDC key. */
		UE_AUDIO_OPUS_VER = 13,
	};

public:
	bool AllowParallelBuild() const override
	{
		return true;
	}

	uint16 GetVersion(FName Format) const override
	{
		check(Format == NAME_OPUS);
		return UE_AUDIO_OPUS_VER;
	}


	void GetSupportedFormats(TArray<FName>& OutFormats) const override
	{
		OutFormats.Add(NAME_OPUS);
	}

	bool Cook(FName Format, const TArray<uint8>& SrcBuffer, FSoundQualityInfo& QualityInfo, TArray<uint8>& CompressedDataStore) const override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioFormatOpus::Cook);
		check(Format == NAME_OPUS);

		// For audio encoding purposes we want Full Band encoding with a 20ms frame size.
		const uint32 kOpusSampleRate = GetMatchingOpusSampleRate(QualityInfo.SampleRate);
		const int32 kOpusFrameSizeMs = 20;
		// Calculate frame size required by Opus
		const int32 kOpusFrameSizeSamples = (kOpusSampleRate * kOpusFrameSizeMs) / 1000;
		const uint32 kSampleStride = SAMPLE_SIZE * QualityInfo.NumChannels;
		const int32 kBytesPerFrame = kOpusFrameSizeSamples * kSampleStride;
		// Number of silent samples to prepend that get removed after decoding.
		const int32 kPrerollSkipCount = kOpusFrameSizeSamples * (80 / kOpusFrameSizeMs);	// 80ms worth
		int32 NumPaddingSamplesAtEnd = 0;

		// Remember the actual number of samples
		uint32 TrueSampleCount = SrcBuffer.Num() / kSampleStride;

		// Prepend the initial silence.
		TArray<uint8> SrcBufferCopy = SrcBuffer;
		SrcBufferCopy.InsertZeroed(0, kPrerollSkipCount * SAMPLE_SIZE * QualityInfo.NumChannels);

		// Initialise the Opus encoder
		OpusEncoder* Encoder = NULL;
		int32 EncError = 0;
#if USE_UE_MEM_ALLOC
		int32 EncSize = opus_encoder_get_size(QualityInfo.NumChannels);
		Encoder = (OpusEncoder*)FMemory::Malloc(EncSize);
		EncError = opus_encoder_init(Encoder, kOpusSampleRate, QualityInfo.NumChannels, OPUS_APPLICATION_AUDIO);
#else
		Encoder = opus_encoder_create(kOpusSampleRate, QualityInfo.NumChannels, OPUS_APPLICATION_AUDIO, &EncError);
#endif
		if (EncError != OPUS_OK)
		{
			Destroy(Encoder);
			return false;
		}

		int32 BitRate = GetBitRateFromQuality(QualityInfo);
		opus_encoder_ctl(Encoder, OPUS_SET_BITRATE(BitRate));
		
		// Get the number of pre-skip samples. These are to be skipped in addition to the initial silence.
		int32 PreSkip = 0;
		opus_encoder_ctl(Encoder, OPUS_GET_LOOKAHEAD(&PreSkip));
		// We add silence to the end of the buffer as indicated by the pre-skip so we get this implicit
		// decoder delay accounted for at the end. Otherwise the last samples may not be emitted by the decoder.
		SrcBufferCopy.AddZeroed(PreSkip * SAMPLE_SIZE * QualityInfo.NumChannels);

		// Create a buffer to store compressed data
		CompressedDataStore.Empty();
		FMemoryWriter CompressedData(CompressedDataStore);
		int32 SrcBufferOffset = 0;

		// Calc frame and sample count
		int64 FramesToEncode = SrcBufferCopy.Num() / kBytesPerFrame;
		// Pad the end of data with zeroes if it isn't exactly the size of a frame.
		if (SrcBufferCopy.Num() % kBytesPerFrame != 0)
		{
			int32 FrameDiff = kBytesPerFrame - (SrcBufferCopy.Num() % kBytesPerFrame);
			SrcBufferCopy.AddZeroed(FrameDiff);
			++FramesToEncode;
			NumPaddingSamplesAtEnd = FrameDiff / (SAMPLE_SIZE * QualityInfo.NumChannels);
		}

		check(QualityInfo.NumChannels <= MAX_uint8);
		check(FramesToEncode <= MAX_uint32);
		FOpusAudioInfo::FHeader Hdr;
		Hdr.NumChannels = QualityInfo.NumChannels;
		Hdr.SampleRate = QualityInfo.SampleRate;
		Hdr.EncodedSampleRate = kOpusSampleRate;
		Hdr.ActiveSampleCount = TrueSampleCount;
		Hdr.NumEncodedFrames = (uint32)FramesToEncode;
		Hdr.NumPreSkipSamples = PreSkip;
		Hdr.NumSilentSamplesAtBeginning = kPrerollSkipCount;
		Hdr.NumSilentSamplesAtEnd = NumPaddingSamplesAtEnd;
		SerializeHeaderData(CompressedData, Hdr);

		// Temporary storage with more than enough to store any compressed frame
		TArray<uint8> TempCompressedData;
		TempCompressedData.AddUninitialized(kBytesPerFrame);

		while (SrcBufferOffset < SrcBufferCopy.Num())
		{
			int32 CompressedLength = opus_encode(Encoder, (const opus_int16*)(SrcBufferCopy.GetData() + SrcBufferOffset), kOpusFrameSizeSamples, TempCompressedData.GetData(), TempCompressedData.Num());

			if (CompressedLength < 0)
			{
				const char* ErrorStr = opus_strerror(CompressedLength);
				UE_LOGF(LogAudio, Warning, "Failed to encode: [%d] %ls", CompressedLength, ANSI_TO_TCHAR(ErrorStr));

				Destroy(Encoder);

				CompressedDataStore.Empty();
				return false;
			}
			else
			{
				// Store frame length and copy compressed data before incrementing pointers
				check(CompressedLength < MAX_uint16);
				SerialiseFrameData(CompressedData, TempCompressedData.GetData(), CompressedLength);
				SrcBufferOffset += kBytesPerFrame;
			}
		}

		Destroy(Encoder);

		return CompressedDataStore.Num() > 0;
	}

	bool CookSurround(FName Format, const TArray<TArray<uint8> >& SrcBuffers, FSoundQualityInfo& QualityInfo, TArray<uint8>& CompressedDataStore) const override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAudioFormatOpus::CookSurround);
		check(Format == NAME_OPUS);

		// For audio encoding purposes we want Full Band encoding with a 20ms frame size.
		const uint32 kOpusSampleRate = GetMatchingOpusSampleRate(QualityInfo.SampleRate);
		const int32 kOpusFrameSizeMs = 20;
		// Calculate frame size required by Opus
		const int32 kOpusFrameSizeSamples = (kOpusSampleRate * kOpusFrameSizeMs) / 1000;
		const uint32 kSampleStride = SAMPLE_SIZE * QualityInfo.NumChannels;
		const int32 kBytesPerFrame = kOpusFrameSizeSamples * kSampleStride;
		// Number of silent samples to prepend that get removed after decoding.
		const int32 kPrerollSkipCount = kOpusFrameSizeSamples * (80 / kOpusFrameSizeMs);	// 80ms worth

		TArray<TArray<uint8>> SrcBufferCopies;
		SrcBufferCopies.AddDefaulted(SrcBuffers.Num());
		for(int32 Index=0; Index<SrcBuffers.Num(); ++Index)
		{
			SrcBufferCopies[Index] = SrcBuffers[Index];
		}

		// Ensure that all channels are the same length
		int32 SourceSize = -1;
		for (int32 Index=0; Index<SrcBufferCopies.Num(); ++Index)
		{
			if (!Index)
			{
				SourceSize = SrcBufferCopies[Index].Num();
			}
			else
			{
				if (SourceSize != SrcBufferCopies[Index].Num())
				{
					return false;
				}
			}
		}
		if (SourceSize <= 0)
		{
			return false;
		}

		// Remember the actual number of samples
		uint32 TrueSampleCount = SourceSize / SAMPLE_SIZE;

		// Prepend the initial silence.
		for (int32 Index = 0; Index < SrcBuffers.Num(); Index++)
		{
			SrcBufferCopies[Index].InsertZeroed(0, kPrerollSkipCount * SAMPLE_SIZE);
		}
		SourceSize += kPrerollSkipCount * SAMPLE_SIZE;

		// Initialise the Opus multistream encoder
		OpusMSEncoder* Encoder = NULL;
		int32 EncError = 0;
		int32 streams = 0;
		int32 coupled_streams = 0;
		// Mapping_family 1 as per https://www.rfc-editor.org/rfc/rfc7845.html#section-5.1.1.2
		int32 mapping_family = 1;
		TArray<uint8> mapping;
		mapping.AddUninitialized(QualityInfo.NumChannels);
#if USE_UE_MEM_ALLOC
		int32 EncSize = opus_multistream_surround_encoder_get_size(QualityInfo.NumChannels, mapping_family);
		Encoder = (OpusMSEncoder*)FMemory::Malloc(EncSize);
		EncError = opus_multistream_surround_encoder_init(Encoder, kOpusSampleRate, QualityInfo.NumChannels, mapping_family, &streams, &coupled_streams, mapping.GetData(), OPUS_APPLICATION_AUDIO);
#else
		Encoder = opus_multistream_surround_encoder_create(kOpusSampleRate, QualityInfo.NumChannels, mapping_family, &streams, &coupled_streams, mapping.GetData(), OPUS_APPLICATION_AUDIO, &EncError);
#endif
		if (EncError != OPUS_OK)
		{
			Destroy(Encoder);
			return false;
		}

		int32 BitRate = GetBitRateFromQuality(QualityInfo);
		opus_multistream_encoder_ctl(Encoder, OPUS_SET_BITRATE(BitRate));

		// Get the number of pre-skip samples. These are to be skipped in addition to the initial silence.
		int32 PreSkip = 0;
		opus_multistream_encoder_ctl(Encoder, OPUS_GET_LOOKAHEAD(&PreSkip));
		// We add silence to the end of the buffer as indicated by the pre-skip so we get this implicit
		// decoder delay accounted for at the end. Otherwise the last samples may not be emitted by the decoder.
		for (int32 Index = 0; Index < SrcBuffers.Num(); Index++)
		{
			SrcBufferCopies[Index].AddZeroed(PreSkip * SAMPLE_SIZE);
		}
		SourceSize += PreSkip * SAMPLE_SIZE;

		// Create a buffer to store compressed data
		CompressedDataStore.Empty();
		FMemoryWriter CompressedData(CompressedDataStore);
		int32 SrcBufferOffset = 0;

		// Calc frame and sample count
		int64 FramesToEncode = SourceSize / (kOpusFrameSizeSamples * SAMPLE_SIZE);
		// Add another frame if Source does not divide into an equal number of frames
		int32 NumSamplesInLastBlock = (SourceSize / SAMPLE_SIZE) % kOpusFrameSizeSamples;
		if (NumSamplesInLastBlock != 0)
		{
			// The silence to pad the last block with is handled in the compression loop below.
			++FramesToEncode;
		}

		check(QualityInfo.NumChannels <= MAX_uint8);
		check(FramesToEncode <= MAX_uint32);

		FOpusAudioInfo::FHeader Hdr;
		Hdr.NumChannels = QualityInfo.NumChannels;
		Hdr.SampleRate = QualityInfo.SampleRate;
		Hdr.EncodedSampleRate = kOpusSampleRate;
		Hdr.ActiveSampleCount = TrueSampleCount;
		Hdr.NumEncodedFrames = (uint32) FramesToEncode;
		Hdr.NumPreSkipSamples = PreSkip;
		Hdr.NumSilentSamplesAtBeginning = kPrerollSkipCount;
		Hdr.NumSilentSamplesAtEnd = NumSamplesInLastBlock ? kOpusFrameSizeSamples - NumSamplesInLastBlock : 0;
		SerializeHeaderData(CompressedData, Hdr);

		// Temporary storage for source data in an interleaved format
		TArray<uint8> TempInterleavedSrc;
		TempInterleavedSrc.AddUninitialized(kBytesPerFrame);

		// Temporary storage with more than enough to store any compressed frame
		TArray<uint8> TempCompressedData;
		TempCompressedData.AddUninitialized(kBytesPerFrame);

		while (SrcBufferOffset < SourceSize)
		{
			// Read a frames worth of data from the source and pack it into interleaved temporary storage
			for (int32 SampleIndex = 0; SampleIndex < kOpusFrameSizeSamples; ++SampleIndex)
			{
				int32 CurrSrcOffset = SrcBufferOffset + SampleIndex*SAMPLE_SIZE;
				int32 CurrInterleavedOffset = SampleIndex*kSampleStride;
				if (CurrSrcOffset < SourceSize)
				{
					check(QualityInfo.NumChannels <= 8); // Static analysis fix: warning C6385: Reading invalid data from 'Order':  the readable size is '256' bytes, but '8160' bytes may be read.
					for (uint32 ChannelIndex = 0; ChannelIndex < QualityInfo.NumChannels; ++ChannelIndex)
					{
						// Interleave the channels in the Vorbis format, so that the correct channel is used for LFE
						int32 OrderedChannelIndex = VorbisChannelInfo::Order[QualityInfo.NumChannels - 1][ChannelIndex];
						int32 CurrInterleavedIndex = CurrInterleavedOffset + ChannelIndex*SAMPLE_SIZE;

						// Copy both bytes that make up a single sample
						TempInterleavedSrc[CurrInterleavedIndex] = SrcBufferCopies[OrderedChannelIndex][CurrSrcOffset];
						TempInterleavedSrc[CurrInterleavedIndex + 1] = SrcBufferCopies[OrderedChannelIndex][CurrSrcOffset + 1];
					}
				}
				else
				{
					// Zero the rest of the temp buffer to make it an exact frame
					FMemory::Memzero(TempInterleavedSrc.GetData() + CurrInterleavedOffset, kBytesPerFrame - CurrInterleavedOffset);
					SampleIndex = kOpusFrameSizeSamples;
				}
			}

			int32 CompressedLength = opus_multistream_encode(Encoder, (const opus_int16*)(TempInterleavedSrc.GetData()), kOpusFrameSizeSamples, TempCompressedData.GetData(), TempCompressedData.Num());

			if (CompressedLength < 0)
			{
				const char* ErrorStr = opus_strerror(CompressedLength);
				UE_LOGF(LogAudio, Warning, "Failed to encode: [%d] %ls", CompressedLength, ANSI_TO_TCHAR(ErrorStr));

				Destroy(Encoder);

				CompressedDataStore.Empty();
				return false;
			}
			else
			{
				// Store frame length and copy compressed data before incrementing pointers
				check(CompressedLength < MAX_uint16);
				SerialiseFrameData(CompressedData, TempCompressedData.GetData(), CompressedLength);
				SrcBufferOffset += kOpusFrameSizeSamples * SAMPLE_SIZE;
			}
		}

		Destroy(Encoder);

		return CompressedDataStore.Num() > 0;
	}

	int32 Recompress(FName Format, const TArray<uint8>& SrcBuffer, FSoundQualityInfo& QualityInfo, TArray<uint8>& OutBuffer) const override
	{
		check(Format == NAME_OPUS);
		FOpusAudioInfo	AudioInfo;

		// Cannot quality preview multichannel sounds
		if( QualityInfo.NumChannels > 2 )
		{
			return 0;
		}

		TArray<uint8> CompressedDataStore;
		if( !Cook( Format, SrcBuffer, QualityInfo, CompressedDataStore ) )
		{
			return 0;
		}

		// Parse the opus header for the relevant information
		if( !AudioInfo.ReadCompressedInfo( CompressedDataStore.GetData(), CompressedDataStore.Num(), &QualityInfo ) )
		{
			return 0;
		}

		// Decompress all the sample data
		OutBuffer.Empty(QualityInfo.SampleDataSize);
		OutBuffer.AddZeroed(QualityInfo.SampleDataSize);
		AudioInfo.ExpandFile( OutBuffer.GetData(), &QualityInfo );

		return CompressedDataStore.Num();
	}

	int32 GetMinimumSizeForInitialChunk(FName Format, const TArray<uint8>& SrcBuffer) const override
	{
		return FOpusAudioInfo::FHeader::HeaderSize();
	}

	bool SplitDataForStreaming(const TArray<uint8>& SrcBuffer, TArray<TArray<uint8>>& OutBuffers, const int32 MaxInitialChunkSize, const int32 MaxChunkSize) const override
	{
		// This should not be called if we require a streaming seek-table. 
		if (!ensure(!RequiresStreamingSeekTable()))
		{
			return false;
		}

		if (SrcBuffer.Num() == 0)
		{
			return false;
		}

		uint32 ReadOffset = 0;
		uint32 WriteOffset = 0;
		uint32 ProcessedFrames = 0;
		const uint8* LockedSrc = SrcBuffer.GetData();

		FOpusAudioInfo::FHeader Hdr;
		if (!FOpusAudioInfo::ParseHeader(Hdr, ReadOffset, LockedSrc, SrcBuffer.Num()))
		{
			return false;
		}

		// Should always be able to store basic info in a single chunk
		check(ReadOffset - WriteOffset <= (uint32)MaxInitialChunkSize);

		int32 ChunkSize = MaxInitialChunkSize;

		while (ProcessedFrames < Hdr.NumEncodedFrames)
		{
			uint16 FrameSize = *((uint16*)(LockedSrc + ReadOffset));

			if ( (ReadOffset + sizeof(uint16) + FrameSize) - WriteOffset >= ChunkSize)
			{
				WriteOffset += AddDataChunk(OutBuffers, LockedSrc + WriteOffset, ReadOffset - WriteOffset);
			}

			ReadOffset += sizeof(uint16) + FrameSize;
			ProcessedFrames++;

			ChunkSize = MaxChunkSize;
		}
		if (WriteOffset < ReadOffset)
		{
			WriteOffset += AddDataChunk(OutBuffers, LockedSrc + WriteOffset, ReadOffset - WriteOffset);
		}

		return true;
	}

	bool RequiresStreamingSeekTable() const override
	{
		return true;
	}

	bool ExtractSeekTableForStreaming(TArray<uint8>& InOutBuffer, IAudioFormat::FSeekTable& OutSeektable) const override
	{
		// This should only be called if we require a streaming seek-table. 
		if (!ensure(RequiresStreamingSeekTable()))
		{
			return false;
		}

		FOpusAudioInfo::FHeader Hdr;
		uint32 CurrentOffset = 0;
		uint8* Data = InOutBuffer.GetData();
		if (!FOpusAudioInfo::ParseHeader(Hdr, CurrentOffset, Data, InOutBuffer.Num()))
		{
			return false;
		}
		Data += CurrentOffset;
		int32 NumFrames = Hdr.NumEncodedFrames;
		OutSeektable.Offsets.SetNum(NumFrames);
		OutSeektable.Times.SetNum(NumFrames);

		const uint16 kOpusSampleRate = 48000;
		const int32 kOpusFrameSizeMs = 20;
		const int32 kOpusFrameSizeSamples = (kOpusSampleRate * kOpusFrameSizeMs) / 1000;

		uint64 SamplePos = 0;
		uint32 CompressedPos = CurrentOffset;
		for(int32 i=0; i<NumFrames; ++i)
		{
			OutSeektable.Times[i] = SamplePos;
			OutSeektable.Offsets[i] = CompressedPos;
			uint16 FrameSize = *reinterpret_cast<const uint16*>(Data);
			Data += FrameSize + sizeof(uint16);
			CompressedPos += FrameSize + sizeof(uint16);
			SamplePos += kOpusFrameSizeSamples;
		}
		return true;
	}

	int32 GetBitRateFromQuality(FSoundQualityInfo& QualityInfo) const
	{
		const int32 kMinBpsPerChannel = 16000;
		const int32 kMaxBpsPerChannel = 96000;
		const int32 kQuality = QualityInfo.Quality < 1 ? 1 : QualityInfo.Quality > 100 ? 100 : QualityInfo.Quality;
		int32 Bps = (int32) FMath::GetMappedRangeValueClamped(FVector2f(1, 100), FVector2f(kMinBpsPerChannel, kMaxBpsPerChannel), (float)kQuality);
		Bps *= QualityInfo.NumChannels;
		return Bps;
	}

	uint32 GetMatchingOpusSampleRate(uint32 InRate) const
	{
		if (InRate <= 8000)
		{
			return 8000;
		}
		else if (InRate <= 12000)
		{
			return 12000;
		}
		if (InRate <= 16000)
		{
			return 16000;
		}
		if (InRate <= 24000)
		{
			return 24000;
		}
		return 48000;
	}


	void SerializeHeaderData(FMemoryWriter& CompressedData, FOpusAudioInfo::FHeader& InHeader) const
	{
		InHeader.Version = 0;
		FMemory::Memcpy(InHeader.Identifier, FOpusAudioInfo::FHeader::OPUS_ID, 8);
		CompressedData.Serialize(InHeader.Identifier, 8);
		CompressedData.Serialize(&InHeader.Version, sizeof(uint8));
		CompressedData.Serialize(&InHeader.NumChannels, sizeof(uint8));
		CompressedData.Serialize(&InHeader.SampleRate, sizeof(uint32));
		CompressedData.Serialize(&InHeader.EncodedSampleRate, sizeof(uint32));
		CompressedData.Serialize(&InHeader.ActiveSampleCount, sizeof(uint64));
		CompressedData.Serialize(&InHeader.NumEncodedFrames, sizeof(uint32));
		CompressedData.Serialize(&InHeader.NumPreSkipSamples, sizeof(int32));
		CompressedData.Serialize(&InHeader.NumSilentSamplesAtBeginning, sizeof(int32));
		CompressedData.Serialize(&InHeader.NumSilentSamplesAtEnd, sizeof(int32));
	}

	void SerialiseFrameData(FMemoryWriter& CompressedData, uint8* FrameData, uint16 FrameSize) const
	{
		CompressedData.Serialize(&FrameSize, sizeof(uint16));
		CompressedData.Serialize(FrameData, FrameSize);
	}

	void Destroy(OpusEncoder* Encoder) const
	{
#if USE_UE_MEM_ALLOC
		FMemory::Free(Encoder);
#else
		opus_encoder_destroy(Encoder);
#endif
	}

	void Destroy(OpusMSEncoder* Encoder) const
	{
#if USE_UE_MEM_ALLOC
		FMemory::Free(Encoder);
#else
		opus_multistream_encoder_destroy(Encoder);
#endif
	}

	/**
	 * Adds a new chunk of data to the array
	 *
	 * @param	OutBuffers	Array of buffers to add to
	 * @param	ChunkData	Pointer to chunk data
	 * @param	ChunkSize	How much data to write
	 * @return	How many bytes were written
	 */
	int32 AddDataChunk(TArray<TArray<uint8>>& OutBuffers, const uint8* ChunkData, int32 ChunkSize) const
	{
		TArray<uint8>& NewBuffer = OutBuffers.AddDefaulted_GetRef();
		NewBuffer.Empty(ChunkSize);
		NewBuffer.AddUninitialized(ChunkSize);
		FMemory::Memcpy(NewBuffer.GetData(), ChunkData, ChunkSize);
		return ChunkSize;
	}
};


/**
 * Module for opus audio compression
 */

static IAudioFormat* Singleton = NULL;

class FAudioPlatformOpusModule : public IAudioFormatModule
{
public:
	virtual ~FAudioPlatformOpusModule()
	{
		delete Singleton;
		Singleton = NULL;
	}
	virtual IAudioFormat* GetAudioFormat()
	{
		if (!Singleton)
		{
			Singleton = new FAudioFormatOpus();
		}
		return Singleton;
	}
};

IMPLEMENT_MODULE( FAudioPlatformOpusModule, AudioFormatOpus);
